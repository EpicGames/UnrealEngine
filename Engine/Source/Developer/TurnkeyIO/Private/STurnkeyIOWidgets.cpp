// Copyright Epic Games, Inc. All Rights Reserved.

#include "STurnkeyIOWidgets.h"

#if WITH_TURNKEY_EDITOR_IO_SERVER
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Dom/JsonObject.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "TurnkeySlateIO"



void STurnkeyReadInputModal::Construct( const FArguments& InArgs )
{
	// store arguments
	Prompt = InArgs._Prompt;
	Default = InArgs._Default;
	OnFinished = InArgs._OnFinished;
	Value = Default;

	// construct widget
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(8)
		[
			SNew(SVerticalBox)

			// prompt 
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4,4)
			[
				SNew(STextBlock)
				.Text( FText::FromString(Prompt) )
			]

			// text input
			+SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(4,4)
			.VAlign(VAlign_Top)
			[
				SNew(SEditableTextBox)
				.Text_Lambda( [this]() { return FText::FromString(Value); } )
				.OnTextChanged_Lambda( [this]( const FText& NewText ) { Value = NewText.ToString(); } )
				.OnTextCommitted_Lambda( [this]( const FText& NewText, ETextCommit::Type CommitType )
				{
					if (CommitType == ETextCommit::OnEnter)
					{
						FinishAction();
					}
				})
			]

			// button area
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				// ok button
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(5,5)
				.AutoWidth()
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OK", "OK"))
					.OnClicked_Lambda( [this]()
					{ 
						FinishAction(); 
						return FReply::Handled();
					})
				]
			]
		]
	];
}


void STurnkeyReadInputModal::FinishAction()
{
	TSharedPtr<FJsonObject> Result = MakeShared<FJsonObject>();
	Result->SetStringField( TEXT("Result"), Value );

	OnFinished.ExecuteIfBound(Result);
}






void STurnkeyReadInputIntModal::Construct( const FArguments& InArgs )
{
	// store arguments
	Prompt = InArgs._Prompt;
	Options = InArgs._Options;
	bIsCancellable = InArgs._IsCancellable;
	DefaultValue = InArgs._DefaultValue;
	OnFinished = InArgs._OnFinished;
	Value = DefaultValue;

	// construct widget
	TSharedPtr<SVerticalBox> VerticalBox;
	TSharedPtr<SHorizontalBox> ButtonBox;
	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(8)
		[
			SNew(SVerticalBox)

			// prompt
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4,4)
			[
				SNew(STextBlock)
				.Text( FText::FromString(Prompt) )
			]

			// container widget 
			+SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(16,16)
			[
				SAssignNew(VerticalBox, SVerticalBox)
			]

			// button area
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				SAssignNew(ButtonBox, SHorizontalBox)

				// ok button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5,5)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("OK", "OK"))
					.IsEnabled_Lambda( [this]() { return Value > 0; } )
					.OnClicked_Lambda( [this]()
					{ 
						FinishAction();
						return FReply::Handled();
					})
				]
			]
		]
	];

	// add cancel button, if permitted
	if ( bIsCancellable )
	{
		ButtonBox->AddSlot()
		.AutoWidth()
		.Padding(5,5)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "Button")
			.TextStyle( FAppStyle::Get(), "DialogButtonText" )
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Text(LOCTEXT("Cancel", "Cancel"))
			.OnClicked_Lambda( [this]()
			{ 
				Value = 0;
				FinishAction();
				return FReply::Handled();
			})
		];
	}

	// add all options
	int ResultValue = 1;
	for( int OptionIndex = 0; OptionIndex < Options.Num(); OptionIndex++ )
	{
		FString OptionValue = Options[OptionIndex];
		if( OptionValue.RemoveFromStart(TEXT(";")) )
		{
			// just a comment - not a selectable option. create a heading
			VerticalBox->AddSlot()
			.Padding(4,4)
			[
				SNew(STextBlock)
				.Text( FText::FromString(OptionValue) )
			];
		}
		else
		{
			TSharedPtr<STextBlock> TextBlock;

			// create a radio button & label
			VerticalBox->AddSlot()
			.Padding(4,4)
			[
				// radio button
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				[
					SNew(SCheckBox)
					.Style(FAppStyle::Get(), "RadioButton")
					.IsChecked_Lambda( [this, ResultValue]() { return (Value == ResultValue) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
					.OnCheckStateChanged_Lambda([this, ResultValue](const ECheckBoxState NewState)
					{
						if (NewState == ECheckBoxState::Checked)
						{
							Value = ResultValue;
						}
					})

				]

				// label
				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Top)
				[
					SAssignNew(TextBlock, STextBlock)
					.Text( FText::FromString(OptionValue) )
					.OnDoubleClicked_Lambda([this, ResultValue](const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
					{
						// double click the label to select the option immediately
						Value = ResultValue;
						FinishAction();
						return FReply::Handled();
					})

				]
			];

			// change the option if the user clicks on the label
			TextBlock->SetOnMouseButtonUp( FPointerEventHandler::CreateLambda( [this, ResultValue](const FGeometry& MyGeometry, const FPointerEvent& PointerEvent)
			{
				Value = ResultValue;
				return FReply::Handled();
			}));


			// update the result value for next time
			ResultValue++;
		}
	};
}


void STurnkeyReadInputIntModal::FinishAction()
{
	TSharedPtr<FJsonObject> Result = MakeShared<FJsonObject>();
	Result->SetNumberField( TEXT("Result"), static_cast<double>(Value) );

	OnFinished.ExecuteIfBound(Result);
}



#undef LOCTEXT_NAMESPACE
#endif //WITH_TURNKEY_EDITOR_IO_SERVER
