// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TurnkeyEditorIOServer.h"

#if WITH_TURNKEY_EDITOR_IO_SERVER
#include "Widgets/SCompoundWidget.h"

DECLARE_DELEGATE_OneParam( FOnTurnkeyActionComplete, TSharedPtr<class FJsonObject> );


class STurnkeyReadInputModal : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( STurnkeyReadInputModal ){}
	SLATE_ARGUMENT( FString, Prompt );
	SLATE_ARGUMENT( FString, Default );
	SLATE_EVENT( FOnTurnkeyActionComplete, OnFinished );
	SLATE_END_ARGS();

public:
	void Construct( const FArguments& InArgs );

private:
	void FinishAction();

	FOnTurnkeyActionComplete OnFinished;
	FString Prompt;
	FString Default;
	FString Value;
};



class STurnkeyReadInputIntModal : public SCompoundWidget
{
	SLATE_BEGIN_ARGS( STurnkeyReadInputIntModal ){}
	SLATE_ARGUMENT( FString, Prompt );
	SLATE_ARGUMENT( TArray<FString>, Options );
	SLATE_ARGUMENT( bool, IsCancellable );
	SLATE_ARGUMENT( int, DefaultValue );
	SLATE_EVENT( FOnTurnkeyActionComplete, OnFinished );
	SLATE_END_ARGS();

public:
	void Construct( const FArguments& InArgs );

private:
	void FinishAction();

	FString Prompt;
	TArray<FString> Options;
	bool bIsCancellable;
	int DefaultValue;
	int Value;
	FOnTurnkeyActionComplete OnFinished;
};


#endif //WITH_TURNKEY_EDITOR_IO_SERVER

