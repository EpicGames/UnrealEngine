// Copyright Epic Games, Inc. All Rights Reserved.

#include "TurnkeyEditorIOServer.h"

#if WITH_TURNKEY_EDITOR_IO_SERVER
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "HAL/Event.h"
#include "Misc/MessageDialog.h"
#include "Async/Async.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Application/SlateApplication.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "STurnkeyIOWidgets.h"

#define LOCTEXT_NAMESPACE "TurnkeyEditorIOServer"

DEFINE_LOG_CATEGORY_STATIC(LogTurnkeyIO, Log, All);


class FTurnkeyEditorIOServerRunnable : public FRunnable
{
public:
	static const int Version = 1; //keep in sync with .cs


	FTurnkeyEditorIOServerRunnable(TSharedPtr<FSocket> InServerSocket)
		: ServerSocket(InServerSocket)
	{
		ReceiveBuffer.AddZeroed(10240);
		ActionCompleteEvent = FPlatformProcess::GetSynchEventFromPool(true);
	}


	~FTurnkeyEditorIOServerRunnable()
	{
		FPlatformProcess::ReturnSynchEventToPool(ActionCompleteEvent);
		ActionCompleteEvent = nullptr;
	}


	virtual uint32 Run() override
	{
		while(!IsEngineExitRequested() && ServerSocket.IsValid())
		{
			// wait for a connection
			UE_LOGF( LogTurnkeyIO, Verbose, "Turnkey IO waiting for connection...");
			ClientSocket = TSharedPtr<FSocket>( ServerSocket->Accept(TEXT("TurnkeyIOClient")) );
			if (!IsClientConnected())
			{
				UE_LOGF( LogTurnkeyIO, Verbose, "Client not connected");
				continue;
			}
			UE_LOGF( LogTurnkeyIO, Verbose, "Client connected");

			// read the message
			TSharedPtr<FJsonObject> Message = ReceiveMessage();
			if (Message.IsValid())
			{
				TSharedPtr<FJsonObject> Action = Message->GetObjectField(TEXT("Action"));
				if (Action.IsValid())
				{
					HandleActionMessage(Action);
				}
			}

			// clean up for next time
			UE_LOGF( LogTurnkeyIO, Verbose, "Transaction complete. Disconnecting" );
			if (ClientSocket)
			{
				ClientSocket->Close();
				ClientSocket.Reset();
			}

			// wait a short while before reading another message, to give the message log time to display any output from Turnkey
			if (ServerSocket.IsValid())
			{
				FPlatformProcess::Sleep(0.1f);
			}
		}

		return 0;
	}


	virtual void Stop() override
	{
		if (ServerSocket.IsValid())
		{
			ServerSocket->Close();
			ServerSocket.Reset();
		}
		if (ClientSocket.IsValid())
		{
			ClientSocket->Close();
			ClientSocket.Reset();
		}
		if (ActionCompleteEvent)
		{
			ActionCompleteEvent->Trigger();
		}
	}

private:

	void OnActionFinish( TSharedPtr<FJsonObject> Result )
	{
		if (Window.IsValid())
		{
			TSharedPtr<SWindow> OldWindow = Window;
			Window.Reset();
			OldWindow->RequestDestroyWindow();
		}

		UE_LOGF( LogTurnkeyIO, Verbose, "Turnkey action completed. Sending reply.");
		SendMessage(Result);

		UE_LOGF( LogTurnkeyIO, Verbose, "Turnkey awaiting ack");
		TSharedPtr<FJsonObject> Ack = ReceiveMessage();

		ActionCompleteEvent->Trigger();
	}


	void HandleActionMessage( TSharedPtr<FJsonObject> Action )
	{
		// read action type
		FString Type;
		if (!Action->TryGetStringField(TEXT("Type"), Type ))
		{
			UE_LOGF( LogTurnkeyIO, Error, "Invalid action message" );
			return;
		}

		// check the action type
		ActionCompleteEvent->Reset();
		if (Type.Equals(TEXT("PauseForUser"), ESearchCase::IgnoreCase ) )
		{
			DoPauseForUser(Action);
		}
		else if (Type.Equals(TEXT("ReadInput"), ESearchCase::IgnoreCase ) )
		{
			DoReadInput(Action);
		}
		else if (Type.Equals(TEXT("ReadInputInt"), ESearchCase::IgnoreCase ) )
		{
			DoReadInputInt(Action);
		}
		else if (Type.Equals(TEXT("GetConfirmation"), ESearchCase::IgnoreCase ) )
		{
			DoGetUserConfirmation(Action);
		}
		else
		{
			UE_LOGF(LogTurnkeyIO, Error, "Unknown action type %ls", *Type );
			return;
		}

		// wait for the action to finish
		UE_LOGF( LogTurnkeyIO, Verbose, "Turnkey action in progress. Waiting for completion" );
		ActionCompleteEvent->Wait();
	}


	void DoPauseForUser( TSharedPtr<FJsonObject> JsonObject )
	{
		AsyncTask( ENamedThreads::GameThread, [this, JsonObject]()
		{
			// read parameters
			FString Message;
			JsonObject->TryGetStringField(TEXT("Message"), Message );

			// display the message box
			FString Prompt = Message.IsEmpty() ? *LOCTEXT("ReadyToContinue", "Click OK To Continue").ToString() : Message;
			FMessageDialog::Open(EAppMsgType::Ok, FText::FromString(Prompt), LOCTEXT("Turnkey", "Turnkey") );

			// send empty response
			TSharedPtr<FJsonObject> Result = MakeShared<FJsonObject>();
			OnActionFinish( Result );
		});
	}


	void DoReadInput( TSharedPtr<FJsonObject> JsonObject )
	{
		AsyncTask( ENamedThreads::GameThread, [this, JsonObject]()
		{
			// read parameters
			FString Prompt;
			JsonObject->TryGetStringField(TEXT("Prompt"), Prompt);

			FString Default;
			JsonObject->TryGetStringField(TEXT("Default"), Default );

			// show the dialog
			ShowModal( SNew(STurnkeyReadInputModal)
				.Prompt(Prompt)
				.Default(Default)
				.OnFinished( FOnTurnkeyActionComplete::CreateRaw( this, &FTurnkeyEditorIOServerRunnable::OnActionFinish) )
			);
		});

	}


	void DoReadInputInt( TSharedPtr<FJsonObject> JsonObject )
	{
		AsyncTask( ENamedThreads::GameThread, [this, JsonObject]()
		{
			// read parameters
			FString Prompt;
			JsonObject->TryGetStringField(TEXT("Prompt"), Prompt);

			TArray<FString> Options;
			JsonObject->TryGetStringArrayField(TEXT("Options"), Options);

			bool bIsCancellable = true;
			JsonObject->TryGetBoolField(TEXT("IsCancellable"), bIsCancellable);

			int32 DefaultValue = 0;
			JsonObject->TryGetNumberField(TEXT("DefaultValue"), DefaultValue);

			// show the dialog
			ShowModal( SNew(STurnkeyReadInputIntModal)
				.Prompt(Prompt)
				.Options(Options)
				.IsCancellable(bIsCancellable)
				.DefaultValue(DefaultValue)
				.OnFinished( FOnTurnkeyActionComplete::CreateRaw( this, &FTurnkeyEditorIOServerRunnable::OnActionFinish) )
			);
		});
	}


	void DoGetUserConfirmation( TSharedPtr<FJsonObject> JsonObject )
	{
		AsyncTask( ENamedThreads::GameThread, [this, JsonObject]()
		{
			// read parameters
			FString Message;
			JsonObject->TryGetStringField(TEXT("Message"), Message );

			bool bDefaultValue = true;
			JsonObject->TryGetBoolField(TEXT("DefaultValue"), bDefaultValue );

			// display the message box
			EAppReturnType::Type Result = (bDefaultValue ? EAppReturnType::Yes : EAppReturnType::No );
			Result = FMessageDialog::Open(EAppMsgType::YesNo, Result, FText::FromString(Message), LOCTEXT("Turnkey", "Turnkey") );

			// send empty response
			TSharedPtr<FJsonObject> JsonResult = MakeShared<FJsonObject>();
			JsonResult->SetBoolField(TEXT("Result"), (Result == EAppReturnType::Yes) );
			OnActionFinish( JsonResult );
		});
	}


	void ShowModal( TSharedRef<SWidget> Modal )
	{
		// create a window for the widget
		Window = SNew(SWindow)
			.Title( LOCTEXT("Turnkey","Turnkey") )
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.HasCloseButton(false)
			.AutoCenter(EAutoCenter::PreferredWorkArea)
			.SizingRule(ESizingRule::Autosized)
			[
				Modal
			];

		// even though HasCloseButton is false, the window can still be closed via Windows' alt-tab screen
		Window->GetOnWindowClosedEvent().AddLambda( [this]( const TSharedRef<SWindow>& InWindow )
		{
			if (Window.IsValid())
			{
				Window.Reset();
		
				TSharedPtr<FJsonObject> Result = MakeShared<FJsonObject>();
				Result->SetStringField(TEXT("Error"), TEXT("Turnkey window was closed"));
				OnActionFinish(Result);
			}
		});
		
		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		FSlateApplication::Get().AddModalWindow(Window.ToSharedRef(), RootWindow);
	}


	TSharedPtr<FJsonObject> ReceiveMessage()
	{
		UE_LOGF(LogTurnkeyIO, Verbose, "Waiting for message from Turnkey client" );

		// read the message
		bool bFinished = false;
		TArray<uint8> Msg;
		int RecvFails = 0;
		do
		{
			if (!IsClientConnected())
			{
				return nullptr;
			}

			if (ClientSocket->Wait(ESocketWaitConditions::WaitForRead, FTimespan::FromMilliseconds(100)))
			{
				int32 BytesRead = 0;
				if (ClientSocket->Recv(ReceiveBuffer.GetData(), ReceiveBuffer.Num()-1, BytesRead))
				{
					Msg.Append( ReceiveBuffer.GetData(), BytesRead );
				}
				else
				{
					RecvFails++;

					static const int MaxFails = 100; // arbirary, just to make sure it has definititely failed for good
					if (RecvFails > MaxFails)
					{
						UE_LOGF(LogTurnkeyIO, Error, "Client not responding");
						return TSharedPtr<FJsonObject>();
					}
				}
			}

		} while( !Msg.Contains('\0') );

		// decode the json object
		TSharedPtr<FJsonObject> JsonObject;
		TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(ANSI_TO_TCHAR((const ANSICHAR*)Msg.GetData()));
		if (!FJsonSerializer::Deserialize(Reader, JsonObject) || !JsonObject.IsValid())
		{
			UE_LOGF(LogTurnkeyIO, Error, "Failed to parse message from Turnkey client %ls", *Reader->GetErrorMessage());
			return TSharedPtr<FJsonObject>();
		}
		return JsonObject;
	}


	bool SendMessage( TSharedPtr<FJsonObject> JsonObject )
	{
		if (!IsClientConnected())
		{
			return false;
		}

		// add version to message
		JsonObject->SetNumberField( TEXT("version"), static_cast<double>(Version) );

		// encode the json object and append the message terminator
		FString JsonString;
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonString);
		if (!FJsonSerializer::Serialize( JsonObject.ToSharedRef(), Writer ))
		{
			return false;
		}

		// send the message
		auto Msg = StringCast<ANSICHAR>(*JsonString);
		int32 BytesToSend = Msg.Length()+1; //also send null terminator
		const ANSICHAR* Data = Msg.Get();
		while (BytesToSend > 0)
		{
			while (!ClientSocket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::FromSeconds(1.0)))
			{
				if (ClientSocket->GetConnectionState() == SCS_ConnectionError)
				{
					return false;
				}
			}

			int32 BytesSent;
			if (!ClientSocket->Send( (const uint8*)Data, BytesToSend, BytesSent ))
			{
				return false;
			}

			BytesToSend -= BytesSent;
			Data += BytesSent;
		}
		return true;
	}


	bool IsClientConnected() const
	{
		return ClientSocket.IsValid() && ClientSocket->GetConnectionState() == SCS_Connected;
	}


	FEvent* ActionCompleteEvent;
	TArray<uint8> ReceiveBuffer;
	TSharedPtr<FSocket> ClientSocket;
	TSharedPtr<FSocket> ServerSocket;
	TSharedPtr<SWindow> Window;

};








FTurnkeyEditorIOServer::FTurnkeyEditorIOServer()
	: Port(0)
	, Runnable(nullptr)
	, Thread(nullptr)
{
	Start();
}


FTurnkeyEditorIOServer::~FTurnkeyEditorIOServer()
{
	Stop();
}


FString FTurnkeyEditorIOServer::GetUATParams() const
{
	FString Result;
	if (Thread != nullptr)
	{
		Result += FString::Printf(TEXT("-EditorIOPort=%d "), Port);
	}

	return Result;
}


bool FTurnkeyEditorIOServer::Start()
{
	if (Thread == nullptr)
	{
		ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM);
		if (SocketSubsystem)
		{
			// create an IPv4 TCP server on localhost
			FName ProtocolType = FNetworkProtocolTypes::IPv4;
			TSharedPtr<FSocket> NewSocket( SocketSubsystem->CreateSocket(NAME_Stream, TEXT("TurnkeyIOServer"), ProtocolType ) );
			if (NewSocket.IsValid())
			{
				TSharedRef<FInternetAddr> LocalHostAddr = SocketSubsystem->CreateInternetAddr(ProtocolType);
				LocalHostAddr->SetLoopbackAddress();
				LocalHostAddr->SetPort(0);
				if (NewSocket->Bind( *LocalHostAddr ))
				{
					if (NewSocket->Listen(1))
					{
						Port = NewSocket->GetPortNo();
						Socket = NewSocket;

						Runnable = new FTurnkeyEditorIOServerRunnable(Socket);
						Thread = FRunnableThread::Create( Runnable, TEXT("TurnkeyEditorIOServer"), 0, EThreadPriority::TPri_BelowNormal );
						check( Thread );

						UE_LOGF(LogTurnkeyIO, Verbose, "using port %d", Port );
					}
				}
			}
		}
	}

	return (Thread != nullptr);
}


void FTurnkeyEditorIOServer::Stop()
{
	if (Socket.IsValid())
	{
		Socket->Close();
		Socket.Reset();
	}

	if (Thread)
	{
		Thread->Kill();
		delete Thread;
		delete Runnable;

		Thread = nullptr;
		Runnable = nullptr;
	}
}


#undef LOCTEXT_NAMESPACE
#endif //WITH_TURNKEY_EDITOR_IO_SERVER
