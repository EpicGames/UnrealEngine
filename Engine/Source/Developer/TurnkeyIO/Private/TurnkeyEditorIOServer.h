// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if !defined(WITH_TURNKEY_EDITOR_IO_SERVER)
	#error WITH_TURNKEY_EDITOR_IO_SERVER not defined!
	#define WITH_TURNKEY_EDITOR_IO_SERVER  1 // for IntelliSense
#endif

#if WITH_TURNKEY_EDITOR_IO_SERVER
#include "Templates/SharedPointer.h"

class FTurnkeyEditorIOServer : public TSharedFromThis<FTurnkeyEditorIOServer>
{
public:
	FTurnkeyEditorIOServer();
	~FTurnkeyEditorIOServer();

	FString GetUATParams() const;

private:
	bool Start();
	void Stop();

	int Port;
	class FTurnkeyEditorIOServerRunnable* Runnable;
	class FRunnableThread* Thread;
	TSharedPtr<class FSocket> Socket;

};
#endif //WITH_TURNKEY_EDITOR_IO_SERVER

