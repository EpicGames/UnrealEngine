// Copyright Epic Games, Inc. All Rights Reserved.

#include "ITurnkeyIOModule.h"
#include "TurnkeyEditorIOServer.h"

class FTurnkeyIOModule : public ITurnkeyIOModule
{
public:
	virtual void StartupModule() override
	{
#if WITH_TURNKEY_EDITOR_IO_SERVER
		TurnkeyEditorIOServer = MakeShared<FTurnkeyEditorIOServer>();
#endif
	}

	virtual void ShutdownModule() override
	{
#if WITH_TURNKEY_EDITOR_IO_SERVER
		TurnkeyEditorIOServer.Reset();
#endif
	}


	virtual FString GetUATParams() const override
	{
		FString Result = TEXT("-EditorIO ");

#if WITH_TURNKEY_EDITOR_IO_SERVER
		if (TurnkeyEditorIOServer.IsValid())
		{
			Result += TurnkeyEditorIOServer->GetUATParams();
		}
#endif

		return Result;
	}

private:

#if WITH_TURNKEY_EDITOR_IO_SERVER
	TSharedPtr<FTurnkeyEditorIOServer> TurnkeyEditorIOServer;
#endif

};


IMPLEMENT_MODULE(FTurnkeyIOModule, TurnkeyIO);

