// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * Interface for Turnkey IO module, used for when Turnkey requires user input
 */
class ITurnkeyIOModule : public IModuleInterface
{
public:
	/** 
	 * Returns the command line arguments for passing to UAT
	 */
	virtual FString GetUATParams() const = 0;

	/**
	 * Returns a reference to the Turnkey IO module instance
	 */
	static ITurnkeyIOModule& Get()
	{
		static const FName TurnkeyIOModuleName = "TurnkeyIO";
		return FModuleManager::LoadModuleChecked<ITurnkeyIOModule>(TurnkeyIOModuleName);
	}
};
