// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TurnkeyIO : ModuleRules
{
	public TurnkeyIO(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		if (Target.bBuildDeveloperTools)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
				"CoreUObject",
				"Sockets",
				"Slate",
				"SlateCore",
				"Json",
				"ToolWidgets",
				}
			);

			PublicDefinitions.Add("WITH_TURNKEY_EDITOR_IO_SERVER=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_TURNKEY_EDITOR_IO_SERVER=0");
		}
	}
}
